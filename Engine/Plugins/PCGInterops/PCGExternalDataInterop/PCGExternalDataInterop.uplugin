{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.2",
	"FriendlyName": "Procedural Content Generation Framework (PCG) External Data Interop",
	"Description": "Extra plugin for Procedural Content Generation Framework interacting with external data formats.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/latest/en-US/procedural-content-generation--framework-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PCGExternalDataInterop",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "PCGExternalDataInteropEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
	],
	"Plugins": [
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "AlembicImporter",
			"Enabled": true
		}
	]
}
