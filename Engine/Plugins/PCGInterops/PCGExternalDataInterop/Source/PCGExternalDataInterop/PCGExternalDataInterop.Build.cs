// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGExternalDataInterop : ModuleRules
	{
		public PCGExternalDataInterop(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"Projects",
					"PCG"
				});

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"UnrealEd",
						"AlembicLib",
						"AlembicLibrary"
					}
				);
			}
		}
	}
}
