// Copyright Epic Games, Inc. All Rights Reserved.

#include "Elements/PCGLoadAlembicElement.h"

#include "PCGComponent.h"
#include "Data/PCGBasePointData.h"
#include "Elements/IO/PCGExternalDataContext.h"
#include "Elements/Metadata/PCGMetadataElementCommon.h"
#include "Metadata/Accessors/IPCGAttributeAccessorTpl.h"
#include "Metadata/Accessors/PCGAttributeAccessorHelpers.h"
#include "Metadata/Accessors/PCGAttributeAccessorKeys.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include <string>

#define LOCTEXT_NAMESPACE "PCGLoadAlembic"

#if WITH_EDITOR
#include "AbcImportSettings.h"
#include "AbcUtilities.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_START
#pragma warning(push)
#pragma warning(disable:4005) // TEXT macro redefinition
#include <Alembic/AbcCoreOgawa/All.h>
#include "Alembic/AbcGeom/All.h"
#include "Alembic/AbcCoreFactory/IFactory.h"
#pragma warning(pop)
THIRD_PARTY_INCLUDES_END

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#endif // WITH_EDITOR

struct FPCGLoadAlembicContext : public FPCGExternalDataContext
{
#if WITH_EDITOR
	// Used for alembic parsing
	/** Factory used to generate objects*/
	Alembic::AbcCoreFactory::IFactory Factory;
	/** Archive-typed ABC file */
	Alembic::Abc::IArchive Archive;
	/** Alembic typed root (top) object*/
	Alembic::Abc::IObject TopObject;
#endif // WITH_EDITOR
};

namespace PCGAlembicInterop
{
#if WITH_EDITOR
	void LoadFromAlembicFile(FPCGLoadAlembicContext* Context, const FString& FileName);
#endif
}

#if WITH_EDITOR
void UPCGLoadAlembicSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPCGLoadAlembicSettings, Setup))
	{
		if (Setup == EPCGLoadAlembicStandardSetup::CitySample)
		{
			SetupFromStandard(Setup);
			Setup = EPCGLoadAlembicStandardSetup::None;
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FText UPCGLoadAlembicSettings::GetDefaultNodeTitle() const
{
	return LOCTEXT("NodeTitle", "Load Alembic");
}

FText UPCGLoadAlembicSettings::GetNodeTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Loads data from an Alembic file");
}
#endif // WITH_EDITOR

FPCGElementPtr UPCGLoadAlembicSettings::CreateElement() const
{
	return MakeShared<FPCGLoadAlembicElement>();
}

void UPCGLoadAlembicSettings::SetupFromStandard(EPCGLoadAlembicStandardSetup InSetup)
{
	if (InSetup == EPCGLoadAlembicStandardSetup::CitySample)
	{
		SetupFromStandard(InSetup, ConversionScale, ConversionRotation, bConversionFlipHandedness, AttributeMapping);
	}
}

void UPCGLoadAlembicSettings::SetupFromStandard(EPCGLoadAlembicStandardSetup InSetup, FVector& InConversionScale, FVector& InConversionRotation, bool& bInConversionFlipHandedness, TMap<FString, FPCGAttributePropertyInputSelector>& InAttributeMapping)
{
	if (InSetup == EPCGLoadAlembicStandardSetup::CitySample)
	{
		InConversionScale = FVector(1.0f, 1.0f, 1.0f);
		InConversionRotation = FVector::ZeroVector;
		bInConversionFlipHandedness = true;

		InAttributeMapping.Reset();

		FPCGAttributePropertyInputSelector PositionSelector;
		PositionSelector.Update(FString(TEXT("$Position.xzy")));

		InAttributeMapping.Add(FString(TEXT("position")), MoveTemp(PositionSelector));

		FPCGAttributePropertyInputSelector ScaleSelector;
		ScaleSelector.Update(FString(TEXT("$Scale.xzy")));

		InAttributeMapping.Add(FString(TEXT("scale")), MoveTemp(ScaleSelector));

		FPCGAttributePropertyInputSelector RotationSelector;
		RotationSelector.Update(FString(TEXT("$Rotation.xzyw")));

		InAttributeMapping.Add(FString(TEXT("orient")), MoveTemp(RotationSelector));
	}
}

void FPCGLoadAlembicElement::GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const
{
	FPCGCrc Crc;
	FPCGExternalDataElement::GetDependenciesCrc(InParams, Crc);

	if (const UPCGLoadAlembicSettings* Settings = Cast<UPCGLoadAlembicSettings>(InParams.Settings))
	{
		const FDateTime FileTimeStamp = IFileManager::Get().GetTimeStamp(*Settings->AlembicFilePath.FilePath);
		if (FileTimeStamp != FDateTime::MinValue())
		{
			Crc.Combine(GetTypeHash(FileTimeStamp));
		}
	}

	OutCrc = Crc;
}

FPCGContext* FPCGLoadAlembicElement::CreateContext()
{
	return new FPCGLoadAlembicContext();
}

bool FPCGLoadAlembicElement::PrepareLoad(FPCGExternalDataContext* InContext) const
{
	check(InContext);
	FPCGLoadAlembicContext* Context = static_cast<FPCGLoadAlembicContext*>(InContext);

	check(Context);
	const UPCGLoadAlembicSettings* Settings = Context->GetInputSettings<UPCGLoadAlembicSettings>();
	check(Settings);

#if WITH_EDITOR
	const FString FileName = Settings->AlembicFilePath.FilePath;
	PCGAlembicInterop::LoadFromAlembicFile(Context, FileName);

	if (!Context->PointDataAccessorsMapping.IsEmpty())
	{
		for (const FPCGExternalDataContext::FPointDataAccessorsMapping& DataMapping : Context->PointDataAccessorsMapping)
		{
			FPCGTaggedData& OutData = Context->OutputData.TaggedData.Emplace_GetRef();
			OutData.Data = DataMapping.Data;
		}

		Context->bDataPrepared = true;
	}
#else
	PCGE_LOG(Error, GraphAndLog, LOCTEXT("NotSupportedInGameMode", "The Load Alembic node is not support in non-editor builds."));
#endif

	return true;
}

bool FPCGLoadAlembicElement::ExecuteLoad(FPCGExternalDataContext* InContext) const
{
	if (!FPCGExternalDataElement::ExecuteLoad(InContext))
	{
		return false;
	}

	// Finally, apply conversion
	check(InContext);
	FPCGLoadAlembicContext* Context = static_cast<FPCGLoadAlembicContext*>(InContext);
	check(Context);
	const UPCGLoadAlembicSettings* Settings = Context->GetInputSettings<UPCGLoadAlembicSettings>();
	check(Settings);

	const FVector& ConversionScale = Settings->ConversionScale;
	const FVector& ConversionRotation = Settings->ConversionRotation;
	const bool bFlipRotationW = Settings->bConversionFlipHandedness;

	const FTransform ConversionTransform(FRotator::MakeFromEuler(ConversionRotation), FVector::ZeroVector, ConversionScale);
	if (!ConversionTransform.Equals(FTransform::Identity) || bFlipRotationW)
	{
		for (const FPCGExternalDataContext::FPointDataAccessorsMapping& DataMapping : Context->PointDataAccessorsMapping)
		{
			UPCGBasePointData* PointData = Cast<UPCGBasePointData>(DataMapping.Data);

			if (!PointData)
			{
				continue;
			}

			TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange(/*bAllocate=*/true);

			for (int32 PointIndex = 0; PointIndex < PointData->GetNumPoints(); ++PointIndex)
			{
				TransformRange[PointIndex] = TransformRange[PointIndex] * ConversionTransform;

				if (bFlipRotationW)
				{
					FQuat Rotation = TransformRange[PointIndex].GetRotation();
					Rotation.W *= -1;
					TransformRange[PointIndex].SetRotation(Rotation);
				}
			}
		}
	}

	return true;
}

namespace PCGAlembicInterop
{

#if WITH_EDITOR
	class FPCGAlembicPositionsAccessor : public IPCGAttributeAccessorT<FPCGAlembicPositionsAccessor>
	{
	public:
		using Type = FVector;
		using Super = IPCGAttributeAccessorT<FPCGAlembicPositionsAccessor>;

		FPCGAlembicPositionsAccessor(Alembic::AbcGeom::IPoints::schema_type::Sample& Sample)
			: Super(/*bInReadOnly=*/true)
			, SamplePtr(Sample.getPositions())
		{
		}

		bool GetRangeImpl(TArrayView<FVector> OutValues, int32 Index, const IPCGAttributeAccessorKeys&) const
		{
			for (int32 i = 0; i < OutValues.Num(); ++i)
			{
				Alembic::Abc::P3fArraySample::value_type Position = (*SamplePtr)[Index + i];
				OutValues[i] = FVector(Position.x, Position.y, Position.z);
			}

			return true;
		}

		bool SetRangeImpl(TArrayView<const FVector>, int32, IPCGAttributeAccessorKeys&, EPCGAttributeAccessorFlags)
		{
			return false;
		}

	private:
		Alembic::Abc::P3fArraySamplePtr SamplePtr;
	};

	template<typename T, typename AbcParamType, int32 Extent, int32 SubExtent, bool bUseCStr>
	class FPCGAlembicAccessor : public IPCGAttributeAccessorT<FPCGAlembicAccessor<T, AbcParamType, Extent, SubExtent, bUseCStr>>
	{
	public:
		using Type = T;
		using Super = IPCGAttributeAccessorT<FPCGAlembicAccessor<T, AbcParamType, Extent, SubExtent, bUseCStr>>;

		FPCGAlembicAccessor(Alembic::AbcGeom::ICompoundProperty& Parameters, const FString& PropName)
			: Super(/*bInReadOnly=*/true)
			, Param(Parameters, std::string(TCHAR_TO_UTF8(*PropName)))
			, SamplePtr(Param.getExpandedValue().getVals())
		{
		}

		bool GetRangeImpl(TArrayView<T> OutValues, int32 Index, const IPCGAttributeAccessorKeys&) const
		{
			for (int32 i = 0; i < OutValues.Num(); ++i)
			{
				T& OutValue = OutValues[i];
				const int32 DataIndex = Index + i;

				if constexpr (bUseCStr && Extent == 1 && SubExtent == 1) // String
				{
					OutValue = (*SamplePtr)[DataIndex].c_str();
				}
				else if constexpr (Extent == 1 && SubExtent == 1) // Scalar
				{
					OutValue = (*SamplePtr)[DataIndex];
				}
				else if constexpr (Extent > 1 && SubExtent == 1)
				{
					for (int32 D = 0; D < Extent; ++D)
					{
						OutValue[D] = (*SamplePtr)[DataIndex][D];
					}
				}
				else if constexpr (Extent == 1 && SubExtent > 1)
				{
					for (int32 D = 0; D < SubExtent; ++D)
					{
						OutValue[D] = (*SamplePtr)[DataIndex * SubExtent + D];
					}
				}
				else
				{
					return false;
				}
			}

			return true;
		}

		bool SetRangeImpl(TArrayView<const T>, int32, IPCGAttributeAccessorKeys&, EPCGAttributeAccessorFlags)
		{
			return false;
		}

	private:
		AbcParamType Param;
		using AbcSamplerType = typename AbcParamType::Sample::samp_ptr_type;
		AbcSamplerType SamplePtr;
	};

	TUniquePtr<const IPCGAttributeAccessor> CreateAlembicPropAccessor(Alembic::AbcGeom::ICompoundProperty& Parameters, Alembic::Abc::PropertyHeader& PropertyHeader, const FString& PropName)
	{
		Alembic::Abc::PropertyType PropType = PropertyHeader.getPropertyType();

		Alembic::Abc::DataType DataType = PropertyHeader.getDataType();
		int TypeExtent = DataType.getExtent();

		FString MetadataExtent(PropertyHeader.getMetaData().get("arrayExtent").c_str());
		const int32 SubExtent = (MetadataExtent.Compare("") == 0 ? 1 : FCString::Atoi(*MetadataExtent));

		if (DataType.getPod() == Alembic::Util::kFloat32POD && (TypeExtent == 1 || SubExtent == 1))
		{
			switch (TypeExtent)
			{
			case 1:
				switch (SubExtent)
				{
				case 1: return MakeUnique<FPCGAlembicAccessor<float, Alembic::AbcGeom::IFloatGeomParam, 1, 1, false>>(Parameters, PropName);
				case 2: return MakeUnique<FPCGAlembicAccessor<FVector2D, Alembic::AbcGeom::IFloatGeomParam, 1, 2, false>>(Parameters, PropName);
				case 3: return MakeUnique<FPCGAlembicAccessor<FVector, Alembic::AbcGeom::IFloatGeomParam, 1, 3, false>>(Parameters, PropName);
				case 4: return MakeUnique<FPCGAlembicAccessor<FVector4, Alembic::AbcGeom::IFloatGeomParam, 1, 4, false>>(Parameters, PropName);
				}
			case 2: return MakeUnique<FPCGAlembicAccessor<FVector2D, Alembic::AbcGeom::IV2fGeomParam, 2, 1, false>>(Parameters, PropName);
			case 3: return MakeUnique<FPCGAlembicAccessor<FVector, Alembic::AbcGeom::IV3fGeomParam, 3, 1, false>>(Parameters, PropName);
			case 4: return MakeUnique<FPCGAlembicAccessor<FVector4, Alembic::AbcGeom::IQuatfGeomParam, 4, 1, false>>(Parameters, PropName);
			}
		}
		else if (DataType.getPod() == Alembic::Util::kFloat64POD && (TypeExtent == 1 || SubExtent == 1))
		{
			switch (TypeExtent)
			{
			case 1:
				switch (SubExtent)
				{
				case 1: return MakeUnique<FPCGAlembicAccessor<double, Alembic::AbcGeom::IDoubleGeomParam, 1, 1, false>>(Parameters, PropName);
				case 2: return MakeUnique<FPCGAlembicAccessor<FVector2D, Alembic::AbcGeom::IDoubleGeomParam, 1, 2, false>>(Parameters, PropName);
				case 3: return MakeUnique<FPCGAlembicAccessor<FVector, Alembic::AbcGeom::IDoubleGeomParam, 1, 3, false>>(Parameters, PropName);
				case 4: return MakeUnique<FPCGAlembicAccessor<FVector4, Alembic::AbcGeom::IDoubleGeomParam, 1, 4, false>>(Parameters, PropName);
				}
			case 2: return MakeUnique<FPCGAlembicAccessor<FVector2D, Alembic::AbcGeom::IV2dGeomParam, 2, 1, false>>(Parameters, PropName);
			case 3: return MakeUnique<FPCGAlembicAccessor<FVector, Alembic::AbcGeom::IV3dGeomParam, 3, 1, false>>(Parameters, PropName);
			case 4: return MakeUnique<FPCGAlembicAccessor<FVector4, Alembic::AbcGeom::IQuatdGeomParam, 4, 1, false>>(Parameters, PropName);
			}
		}
		else if ((TypeExtent == 0 || TypeExtent == 1) && SubExtent == 1) // Scalar types
		{
			switch (DataType.getPod())
			{
			case Alembic::Util::kBooleanPOD: return MakeUnique<FPCGAlembicAccessor<bool, Alembic::AbcGeom::IBoolGeomParam, 1, 1, false>>(Parameters, PropName);
			case Alembic::Util::kInt8POD: return MakeUnique<FPCGAlembicAccessor<int32, Alembic::AbcGeom::ICharGeomParam, 1, 1, false>>(Parameters, PropName);
			case Alembic::Util::kInt16POD: return MakeUnique<FPCGAlembicAccessor<int32, Alembic::AbcGeom::IInt16GeomParam, 1, 1, false>>(Parameters, PropName);
			case Alembic::Util::kInt32POD: return MakeUnique<FPCGAlembicAccessor<int32, Alembic::AbcGeom::IInt32GeomParam, 1, 1, false>>(Parameters, PropName);
			case Alembic::Util::kInt64POD: return MakeUnique<FPCGAlembicAccessor<int64, Alembic::AbcGeom::IInt64GeomParam, 1, 1, false>>(Parameters, PropName);
			case Alembic::Util::kUnknownPOD: // fall-through
			case Alembic::Util::kStringPOD: return MakeUnique<FPCGAlembicAccessor<FString, Alembic::AbcGeom::IStringGeomParam, 1, 1, true>>(Parameters, PropName);
			}
		}

		return nullptr;
	}

	bool CreatePointAccessorAndValidate(FPCGContext* Context, UPCGBasePointData* PointData, const TUniquePtr<const IPCGAttributeAccessor>& AlembicPropAccessor, const FPCGAttributePropertySelector& PointPropertySelector, const FString& PropName, TUniquePtr<IPCGAttributeAccessor>& PointPropertyAccessor)
	{
		if (!PointData || !PointData->Metadata)
		{
			return false;
		}

		if (!AlembicPropAccessor)
		{
			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("AlembicPropertyNotSupported", "Property '{0}' is not of a supported type."), FText::FromString(PropName)));
			return false;
		}

		UPCGMetadata* PointMetadata = PointData->Metadata;

		// Create attribute if needed
		if (PointPropertySelector.GetSelection() == EPCGAttributePropertySelection::Attribute && !PointMetadata->HasAttribute(FName(PropName)))
		{
			auto CreateAttribute = [PointMetadata, &PropName](auto Dummy)
				{
					using AttributeType = decltype(Dummy);
					return PCGMetadataElementCommon::ClearOrCreateAttribute<AttributeType>(PointMetadata, FName(PropName)) != nullptr;
				};

			if (!PCGMetadataAttribute::CallbackWithRightType(AlembicPropAccessor->GetUnderlyingType(), CreateAttribute))
			{
				PCGE_LOG_C(Warning, GraphAndLog, Context, FText::Format(LOCTEXT("FailedToCreateNewAttribute", "Failed to create new attribute '{0}'"), FText::FromString(PropName)));
				return false;
			}
		}

		PointPropertyAccessor = PCGAttributeAccessorHelpers::CreateAccessor(PointData, PointPropertySelector);

		if (!PointPropertyAccessor)
		{
			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("AlembicTargetNotSupported", "Unable to write to target property '{0}.'"), PointPropertySelector.GetDisplayText()));
			return false;
		}

		// Final verification, if we can put the value of input into output
		if (!PCG::Private::IsBroadcastable(AlembicPropAccessor->GetUnderlyingType(), PointPropertyAccessor->GetUnderlyingType()))
		{
			FText InputTypeName = FText::FromString(PCG::Private::GetTypeName(AlembicPropAccessor->GetUnderlyingType()));
			FText OutputTypeName = FText::FromString(PCG::Private::GetTypeName(PointPropertyAccessor->GetUnderlyingType()));

			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("CannotBroadcastTypes", "Cannot convert input type '{0}' into output type '{1}'"), InputTypeName, OutputTypeName));
			return false;
		}

		return true;
	}

	bool ParseAlembicObject(FPCGExternalDataContext* Context, const Alembic::Abc::IObject& Object)
	{
		check(Context);
		const UPCGExternalDataSettings* Settings = Context->GetInputSettings<UPCGExternalDataSettings>();
		check(Settings);

		const Alembic::Abc::MetaData& ObjectMetaData = Object.getMetaData();
		const uint32 NumChildren = Object.getNumChildren();

		bool bHasCreatedData = false;

		if (Alembic::AbcGeom::IPoints::matches(ObjectMetaData))
		{
			Alembic::AbcGeom::IPoints Points = Alembic::AbcGeom::IPoints(Object, Alembic::Abc::kWrapExisting);
			Alembic::AbcGeom::IPoints::schema_type::Sample Sample = Points.getSchema().getValue();

			Alembic::Abc::P3fArraySamplePtr Positions = Sample.getPositions();
			uint32 NumPoints = Positions ? Positions->size() : 0;

			if (NumPoints > 0)
			{
				bHasCreatedData = true;

				// Create point data & mapping
				UPCGBasePointData* PointData = FPCGContext::NewPointData_AnyThread(Context);
				check(PointData);

				UPCGMetadata* PointMetadata = PointData->MutableMetadata();

				FPCGExternalDataContext::FPointDataAccessorsMapping& PointDataAccessorMapping = Context->PointDataAccessorsMapping.Emplace_GetRef();
				PointDataAccessorMapping.Data = PointData;
				PointDataAccessorMapping.Metadata = PointMetadata;
				// We're not going to use the input keys, but we still need to provide something
				PointDataAccessorMapping.RowKeys = MakeUnique<FPCGAttributeAccessorKeysEntries>(PCGInvalidEntryKey);

				PointData->SetNumPoints(NumPoints);

				// If the user has provided a position remapping, don't process points right now, instead push the transformation to the row accessors
				if (const FPCGAttributePropertySelector* RemappedPositions = Settings->AttributeMapping.Find(TEXT("Position")))
				{
					TUniquePtr<const IPCGAttributeAccessor> AlembicPositionAccessor = MakeUnique<FPCGAlembicPositionsAccessor>(Sample);
					TUniquePtr<IPCGAttributeAccessor> PointPropertyAccessor;

					FPCGAttributePropertyOutputSelector PointPositionSelector;
					PointPositionSelector.ImportFromOtherSelector(*RemappedPositions);
					const FString PropName = PointPositionSelector.GetName().ToString();

					if (CreatePointAccessorAndValidate(Context, PointData, AlembicPositionAccessor, PointPositionSelector, PropName, PointPropertyAccessor))
					{
						PointDataAccessorMapping.RowToPointAccessors.Emplace(MoveTemp(AlembicPositionAccessor), MoveTemp(PointPropertyAccessor), PointPositionSelector);
					}
				}
				else // Otherwise, write the positions directly
				{
					TPCGValueRange<FTransform> TransformRange = PointData->GetTransformValueRange(/*bAllocate=*/true);
					for (uint32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
					{
						Alembic::Abc::P3fArraySample::value_type Position = (*Positions)[PointIndex];
						TransformRange[PointIndex].SetLocation(FVector(Position.x, Position.y, Position.z));
					}
				}

				Alembic::AbcGeom::ICompoundProperty Parameters = Points.getSchema().getArbGeomParams();
				if (Parameters.valid())
				{
					for (int Index = 0; Index < Parameters.getNumProperties(); ++Index)
					{
						Alembic::Abc::PropertyHeader PropertyHeader = Parameters.getPropertyHeader(Index);
						FString PropName(PropertyHeader.getName().c_str());

						// We'll parse only properties that affect every point/object
						if (PropertyHeader.getPropertyType() != Alembic::Abc::kArrayProperty &&
							!(PropertyHeader.getPropertyType() == Alembic::Abc::kCompoundProperty && (PropertyHeader.getDataType().getPod() == Alembic::Util::kUnknownPOD || PropertyHeader.getDataType().getPod() == Alembic::Util::kStringPOD)))
						{
							PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("AlembicPropertyType", "Property '{0}' is not supported (expected array)."), FText::FromString(PropName)));
							continue;
						}

						TUniquePtr<const IPCGAttributeAccessor> AlembicPropAccessor = CreateAlembicPropAccessor(Parameters, PropertyHeader, PropName);
						TUniquePtr<IPCGAttributeAccessor> PointPropertyAccessor;

						// Setup attribute property selector
						FPCGAttributePropertySelector PointPropertySelector;
						if (const FPCGAttributePropertySelector* MappedField = Settings->AttributeMapping.Find(PropName))
						{
							PointPropertySelector = *MappedField;
							PropName = PointPropertySelector.GetName().ToString();
						}
						else
						{
							PointPropertySelector.Update(PropName);
						}

						if (CreatePointAccessorAndValidate(Context, PointData, AlembicPropAccessor, PointPropertySelector, PropName, PointPropertyAccessor))
						{
							PointDataAccessorMapping.RowToPointAccessors.Emplace(MoveTemp(AlembicPropAccessor), MoveTemp(PointPropertyAccessor), PointPropertySelector);
						}
					}
				}
			}
		}

		if (NumChildren > 0)
		{
			for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
			{
				bHasCreatedData |= ParseAlembicObject(Context, Object.getChild(ChildIndex));
			}
		}

		return bHasCreatedData;
	}

	void LoadFromAlembicFile(FPCGLoadAlembicContext* Context, const FString& FileName)
	{
		check(Context);

		/** Factory used to generate objects*/
		Alembic::AbcCoreFactory::IFactory& Factory = Context->Factory;
		Alembic::AbcCoreFactory::IFactory::CoreType CompressionType = Alembic::AbcCoreFactory::IFactory::kUnknown;
		/** Archive-typed ABC file */
		Alembic::Abc::IArchive& Archive = Context->Archive;
		/** Alembic typed root (top) object*/
		Alembic::Abc::IObject& TopObject = Context->TopObject;

		Factory.setPolicy(Alembic::Abc::ErrorHandler::kQuietNoopPolicy);
		const size_t ReasonableNumStreams = 12; // This is what we had in Rule Processor historically
		Factory.setOgawaNumStreams(ReasonableNumStreams);

		// Extract Archive and compression type from file
		Archive = Factory.getArchive(TCHAR_TO_UTF8(*FileName), CompressionType);

		if (!Archive.valid())
		{
			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("FailedToOpenAlembic", "Failed to open '{0}': Not a valid Alembic file."), FText::FromString(FileName)));
			return;
		}

		// Get Top/root object
		TopObject = Alembic::Abc::IObject(Archive, Alembic::Abc::kTop);
		if (!TopObject.valid())
		{
			PCGE_LOG_C(Error, GraphAndLog, Context, FText::Format(LOCTEXT("FailedToOpenAlembicRoot", "Failed to import '{0}': Alembic root is not valid."), FText::FromString(FileName)));
			return;
		}

		if (!ParseAlembicObject(Context, TopObject))
		{
			PCGE_LOG_C(Warning, GraphAndLog, Context, FText::Format(LOCTEXT("FailedToCreatePointData", "Import of '{0}' was successful but there is no valid point cloud data in the Alembic file."), FText::FromString(FileName)));
			return;
		}
	}

#endif // WITH_EDITOR

}

#undef LOCTEXT_NAMESPACE