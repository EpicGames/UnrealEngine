// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FPCGExternalDataInteropModule final : public IModuleInterface
{
public:
	//~ IModuleInterface implementation
	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}
	//~ End IModuleInterface implementation
};

IMPLEMENT_MODULE(FPCGExternalDataInteropModule, PCGExternalDataInterop);
