// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/IO/PCGExternalData.h"

#include "PCGLoadAlembicElement.generated.h"

struct FPCGExternalDataContext;

UENUM()
enum class EPCGLoadAlembicStandardSetup : uint8
{
	None = 0,
	CitySample UMETA(Tooltip="Uses the same setup as in the City Sample demo: right handed Y-up and the orient and scale mapping to the rotation and scale, respectively")
};

UCLASS(MinimalAPI, BlueprintType, ClassGroup = (Procedural))
class UPCGLoadAlembicSettings : public UPCGExternalDataSettings
{
	GENERATED_BODY()

public:
	// ~Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// ~End UObject interface

	// ~Begin UPCGSettings interface
#if WITH_EDITOR
	virtual FName GetDefaultNodeName() const override { return FName(TEXT("LoadAlembic")); }
	virtual FText GetDefaultNodeTitle() const override;
	virtual FText GetNodeTooltipText() const override;
#endif

protected:
	virtual FPCGElementPtr CreateElement() const override;
	// ~End UPCGSettings interface

public:
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Alembic", meta = (FilePathFilter = "Alembic files (*.abc)|*.abc", PCG_Overridable))
	FFilePath AlembicFilePath;

	// To prevent a dependency on the alembic editor module in this class, we'll keep around only the types we need
	/** Scale to apply during import. Note that for both Max/Maya presets the value flips the Y axis. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Alembic", meta = (PCG_Overridable))
	FVector ConversionScale = FVector(1.0f, -1.0f, 1.0f);

	/** Rotation in Euler angles applied during import. For Max, use (90, 0, 0). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Alembic", meta = (PCG_Overridable))
	FVector ConversionRotation = FVector::ZeroVector;

	/** When changing handedness, it is sometimes needed to flip the rotation direction */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Alembic", meta = (PCG_Overridable, Tooltip="Flips rotation direction (W), useful together with swizzling"))
	bool bConversionFlipHandedness = false;

	UFUNCTION(BlueprintCallable, Category = "PCG|AlembicImport")
	PCGEXTERNALDATAINTEROP_API void SetupFromStandard(EPCGLoadAlembicStandardSetup InSetup);

	static PCGEXTERNALDATAINTEROP_API void SetupFromStandard(EPCGLoadAlembicStandardSetup InSetup, FVector& InConversionScale, FVector& InConversionRotation, bool& bInConversionFlipHandedness, TMap<FString, FPCGAttributePropertyInputSelector>& InAttributeMapping);

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Alembic", meta = (DisplayName="Setup from standard"))
	EPCGLoadAlembicStandardSetup Setup = EPCGLoadAlembicStandardSetup::None;
#endif
};

class PCGEXTERNALDATAINTEROP_API FPCGLoadAlembicElement : public FPCGExternalDataElement
{
public:
	virtual void GetDependenciesCrc(const FPCGGetDependenciesCrcParams& InParams, FPCGCrc& OutCrc) const override;
protected:
	virtual FPCGContext* CreateContext() override;
	virtual bool PrepareLoad(FPCGExternalDataContext* Context) const override;
	virtual bool ExecuteLoad(FPCGExternalDataContext* Context) const override;
};