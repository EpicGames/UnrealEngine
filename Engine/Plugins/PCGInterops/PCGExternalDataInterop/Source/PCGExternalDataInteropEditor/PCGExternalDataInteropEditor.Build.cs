// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PCGExternalDataInteropEditor : ModuleRules
	{
		public PCGExternalDataInteropEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"Projects",
					"PCG"
				});

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"PCGEditor",
						"PCGExternalDataInterop",
						"UnrealEd",
						"AlembicLib",
						"AlembicLibrary"
					}
				);
			}
		}
	}
}
