// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGLoadAlembic.h"
#include "PCGAssetExporterUtils.h"
#include "PCGGraphExecutionStateInterface.h"
#include "PCGModule.h"

#include "Elements/PCGLoadAlembicElement.h"

void UPCGAlembicToPCGAssetExporter::SerializeMetadata(FArchive& Ar)
{
	FPCGLoadAlembicBPData::StaticStruct()->SerializeItem(Ar, &LoadSettings, nullptr);
}

bool UPCGAlembicToPCGAssetExporter::ExportAsset(const FString& PackageName, UPCGDataAsset* Asset)
{
	check(Asset);
	UPCGLoadAlembicFunctionLibrary::LoadAlembicFileToPCG(LoadSettings, Asset->Data, Asset);
	return true;
}

UPackage* UPCGAlembicToPCGAssetExporter::UpdateAsset(const FAssetData& PCGAsset)
{
	UPCGDataAsset* Asset = Cast<UPCGDataAsset>(PCGAsset.GetAsset());
	if (!Asset)
	{
		UE_LOGF(LogPCG, Error, "Asset '%ls' isn't a PCG data asset or could not be properly loaded.", *PCGAsset.GetObjectPathString());
		return nullptr;
	}

	UPackage* Package = Asset->GetPackage();
	if (!Package)
	{
		UE_LOGF(LogPCG, Error, "Unable to retrieve package from Asset '%ls'.", *PCGAsset.GetObjectPathString());
		return nullptr;
	}

	UPCGLoadAlembicFunctionLibrary::LoadAlembicFileToPCG(LoadSettings, Asset->Data, Asset);

	return Package;
}

void UPCGLoadAlembicFunctionLibrary::ExportAlembicFileToPCG(const FPCGLoadAlembicBPData& InSettings, FPCGAssetExporterParameters Parameters)
{
	UPCGAlembicToPCGAssetExporter* Exporter = NewObject<UPCGAlembicToPCGAssetExporter>(GetTransientPackage());
	check(Exporter);

	Exporter->LoadSettings = InSettings;
	UPCGAssetExporterUtils::CreateAsset(Exporter, Parameters);
}

void UPCGLoadAlembicFunctionLibrary::LoadAlembicFileToPCG(const FPCGLoadAlembicBPData& InSettings, FPCGDataCollection& Data, UObject* TargetOuter)
{
	LoadAlembicFileToPCGInternal(InSettings, Data, TargetOuter);
}

void UPCGLoadAlembicFunctionLibrary::LoadAlembicFileToPCGInternal(const FPCGLoadAlembicBPData& InSettings, FPCGDataCollection& Data, UObject* TargetOuter)
{
	// Simulate execution of the FPCGLoadAlembicElement.
	UPCGLoadAlembicSettings* Settings = NewObject<UPCGLoadAlembicSettings>();
	Settings->AlembicFilePath = InSettings.AlembicFilePath;
	Settings->ConversionScale = InSettings.ConversionSettings.Scale;
	Settings->ConversionRotation = InSettings.ConversionSettings.Rotation;
	Settings->bConversionFlipHandedness = InSettings.bConversionFlipHandedness;
	Settings->AttributeMapping = InSettings.AttributeMapping;

	FPCGLoadAlembicElement Element;
	FPCGDataCollection InputData;
	FPCGContext* Context = Element.Initialize(FPCGInitializeElementParams(&InputData, nullptr, nullptr));
	Context->AsyncState.NumAvailableTasks = 1; // TODO : change this
	FPCGTaggedData& DataSettings = Context->InputData.TaggedData.Emplace_GetRef();
	DataSettings.Data = Settings;

	while (!Element.Execute(Context))
	{
		// Nothing
	}

	Data = Context->OutputData;
	delete Context;

	// Reouter the data if needed
	if (TargetOuter)
	{
		for (const FPCGTaggedData& TaggedData : Data.TaggedData)
		{
			if (TaggedData.Data)
			{
				(const_cast<UPCGData*>(TaggedData.Data.Get()))->Rename(nullptr, TargetOuter, REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			}
		}
	}
}

void UPCGLoadAlembicFunctionLibrary::SetupFromStandard(FPCGLoadAlembicBPData& Data, EPCGLoadAlembicStandardSetup InSetup)
{
	if (InSetup != EPCGLoadAlembicStandardSetup::None)
	{
		Data.ConversionSettings.Preset = EAbcConversionPreset::Custom;
		UPCGLoadAlembicSettings::SetupFromStandard(InSetup, Data.ConversionSettings.Scale, Data.ConversionSettings.Rotation, Data.bConversionFlipHandedness, Data.AttributeMapping);
	}	
}