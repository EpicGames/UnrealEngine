// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGAssetExporter.h"
#include "Elements/PCGLoadAlembicElement.h"
#include "Metadata/PCGAttributePropertySelector.h"

#include "AbcImportSettings.h"

#include "PCGLoadAlembic.generated.h"

USTRUCT(BlueprintType)
struct PCGEXTERNALDATAINTEROPEDITOR_API FPCGLoadAlembicBPData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Alembic", meta = (FilePathFilter = "Alembic files (*.abc)|*.abc", PCG_Overridable))
	FFilePath AlembicFilePath;

	/** Conversion settings that will be applied on the transform only */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	FAbcConversionSettings ConversionSettings;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings", meta = (Tooltip = "Flips rotation direction (W), useful together with swizzling"))
	bool bConversionFlipHandedness = false;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Settings")
	TMap<FString, FPCGAttributePropertyInputSelector> AttributeMapping;
};

UCLASS()
class UPCGAlembicToPCGAssetExporter : public UPCGAssetExporter
{
	GENERATED_BODY()

public:
	//~Begin UPCGAssetExporter interface
	virtual bool ExportAsset(const FString& PackageName, UPCGDataAsset* Asset);
	virtual UPackage* UpdateAsset(const FAssetData& PCGAsset) override;

protected:
	virtual void SerializeMetadata(FArchive& Ar) override;
	//~End UPCGAssetExporter interface

public:
	FPCGLoadAlembicBPData LoadSettings;
};

UCLASS()
class PCGEXTERNALDATAINTEROPEDITOR_API UPCGLoadAlembicFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "PCG|IO")
	static void ExportAlembicFileToPCG(const FPCGLoadAlembicBPData& Settings, FPCGAssetExporterParameters Parameters = FPCGAssetExporterParameters());

	UFUNCTION(BlueprintCallable, Category = "PCG|IO", meta = (DeprecatedFunction, DeprecationMessage="The LoadAlembicFileToPCG function has been replaced by ExportAlembicFileToPCG"))
	static void LoadAlembicFileToPCG(const FPCGLoadAlembicBPData& Settings, FPCGDataCollection& Data, UObject* TargetOuter);

	UFUNCTION(BlueprintCallable, Category = "PCG|IO", meta = (ScriptMethod))
	static void SetupFromStandard(UPARAM(ref) FPCGLoadAlembicBPData& Data, EPCGLoadAlembicStandardSetup InSetup);

protected:
	static void LoadAlembicFileToPCGInternal(const FPCGLoadAlembicBPData& Settings, FPCGDataCollection& Data, UObject* TargetOuter);
};